// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NullInstallBundleManager : ModuleRules
{
	public NullInstallBundleManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"InstallBundleManager"
			}
			);
	}
}