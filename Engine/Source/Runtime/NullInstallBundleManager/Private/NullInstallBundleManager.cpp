// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleManagerInterface.h"
#include "InstallBundleManagerModule.h"
#include "Modules/ModuleManager.h"

class FNullInstallBundleManager final : public IInstallBundleManager
{
	virtual bool HasBundleSource(FInstallBundleSourceType SourceType) const override
	{
		return false;
	}

	virtual FDelegateHandle PushInitErrorCallback(FInstallBundleManagerInitErrorHandler Callback) override
	{
		return FDelegateHandle();
	}

	virtual void PopInitErrorCallback() override
	{
	}

	void PopInitErrorCallback(FDelegateHandle Handle) override
	{
	}

	virtual void PopInitErrorCallback(FDelegateUserObjectConst InUserObject) override
	{
	}

	virtual EInstallBundleManagerInitState GetInitState() const override
	{
		return EInstallBundleManagerInitState::Succeeded;
	}

	virtual TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> RequestUpdateContent(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags Flags, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging, InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext = nullptr) override
	{
		return MakeValue(FInstallBundleRequestInfo());
	}

	virtual FDelegateHandle GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, bool bAddDependencies, FInstallBundleGetContentStateDelegate Callback, FName RequestTag) override
	{
		FInstallBundleCombinedContentState State;
		Callback.ExecuteIfBound(State);
		return Callback.GetHandle();
	}

	virtual void CancelAllGetContentStateRequestsForTag(FName RequestTag) override
	{
	}

	virtual void CancelAllGetContentStateRequests(FDelegateHandle Handle) override
	{
	}

	virtual FDelegateHandle GetInstallState(TArrayView<const FName> BundleNames, bool bAddDependencies, FInstallBundleGetInstallStateDelegate Callback, FName RequestTag = NAME_None) override
	{
		FInstallBundleCombinedInstallState State;
		Callback.ExecuteIfBound(State);
		return Callback.GetHandle();
	}

	virtual TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> GetInstallStateSynchronous(TArrayView<const FName> BundleNames, bool bAddDependencies) const override
	{
		return MakeValue(FInstallBundleCombinedInstallState());
	}

	virtual EInstallBundleResult GetAllMountedBundles(TArray<FName>& OutBundleNames) const override
	{
		return EInstallBundleResult::OK;
	}

	virtual EInstallBundleResult GetAllBundlesWithState(EInstallBundleInstallState InstallState, TArray<FName>& OutBundleNames) const override
	{
		return EInstallBundleResult::OK;
	}

	virtual void CancelAllGetInstallStateRequestsForTag(FName RequestTag) override
	{
	}

	virtual void CancelAllGetInstallStateRequests(FDelegateHandle Handle) override
	{
	}

	virtual TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> RequestReleaseContent(TArrayView<const FName> ReleaseNames, EInstallBundleReleaseRequestFlags Flags, TArrayView<const FName> KeepNames = TArrayView<const FName>(), ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		return MakeValue(FInstallBundleReleaseRequestInfo());
	}

	virtual EInstallBundleResult FlushCache(FInstallBundleSourceOrCache SourceOrCache, FInstallBundleManagerFlushCacheCompleteDelegate Callback, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		Callback.ExecuteIfBound();
		return EInstallBundleResult::OK;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TArray<FInstallBundleCacheStats> GetCacheStats(EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		return TArray<FInstallBundleCacheStats>();
	}

	virtual TOptional<FInstallBundleCacheStats> GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		return TOptional<FInstallBundleCacheStats>();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual TArray<FInstallBundleCacheStats> GetCacheStats(EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		return TArray<FInstallBundleCacheStats>();
	}

	virtual TOptional<FInstallBundleCacheStats> GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) override
	{
		return TOptional<FInstallBundleCacheStats>();
	}

	virtual void RequestRemoveContentOnNextInit(TArrayView<const FName> RemoveNames, TArrayView<const FName> KeepNames = TArrayView<const FName>()) override
	{
	}

	virtual void CancelRequestRemoveContentOnNextInit(TArrayView<const FName> BundleNames) override
	{
	}

	virtual TArray<FName> GetRequestedRemoveContentOnNextInit() const override
	{
		return TArray<FName>();
	}

	virtual void CancelUpdateContent(TArrayView<const FName> BundleNames) override
	{
	}

	virtual void PauseUpdateContent(TArrayView<const FName> BundleNames) override
	{

	}

	virtual void ResumeUpdateContent(TArrayView<const FName> BundleNames) override
	{
	}

	virtual void RequestPausedBundleCallback() override
	{
	}

	virtual TOptional<FInstallBundleProgress> GetBundleProgress(FName BundleName) const override
	{
		return TOptional<FInstallBundleProgress>();
	}

	virtual EInstallBundleRequestFlags GetModifyableContentRequestFlags() const override
	{
		return EInstallBundleRequestFlags::None;
	}

	virtual void UpdateContentRequestFlags(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags) override
	{
	}

	virtual void SetCellularPreference(int32 Value) override
	{
	}

	virtual void SetLimitedDataPreference(int32 Value) override
	{
	}

	virtual void SetCacheSize(FName CacheName, uint64 CacheSize) override
	{
	}

	virtual bool SupportsEarlyStartupPatching() const override
	{
		return false;
	}

	virtual bool IsNullInterface() const override
	{
		return true;
	}

private:
	
};

class FNullInstallBundleManagerModule : public TInstallBundleManagerModule<FNullInstallBundleManager>
{	
};

IMPLEMENT_MODULE(FNullInstallBundleManagerModule, NullInstallBundleManager);
